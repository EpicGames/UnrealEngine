// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DrawPrimitiveDebugger : ModuleRules
	{
		public DrawPrimitiveDebugger(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"RenderCore",
					"Renderer",
					"InputCore",
					"SlateCore",
					"Slate",
					"DeveloperSettings",
					"RHI",
				});

			PrivateIncludePathModuleNames.Add("Settings");

			// TODO: Decide whether Test builds should also remain restricted
			if (Target.bBuildDeveloperTools || (Target.Configuration != UnrealTargetConfiguration.Shipping))
			{
				PrecompileForTargets = PrecompileTargetsType.Any;
			}
		}
	}
}