// Copyright Epic Games, Inc. All Rights Reserved.

#include "DrawPrimitiveDebuggerConfig.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DrawPrimitiveDebuggerConfig)

void UDrawPrimitiveDebuggerUserSettings::SetFontSize(const int32 InFontSize)
{
	UDrawPrimitiveDebuggerUserSettings* Settings = GetMutableDefault<UDrawPrimitiveDebuggerUserSettings>();
	Settings->FontSize = FMath::Clamp(InFontSize, 1, 100);

	const FString ConfigFileName = Settings->GetProjectUserConfigFilename();
	GConfig->SetInt(TEXT("/Script/DrawPrimitiveDebugger.DrawPrimitiveDebuggerUserSettings"), TEXT("FontSize"), Settings->FontSize, *ConfigFileName);
	GConfig->Flush(false);
}

