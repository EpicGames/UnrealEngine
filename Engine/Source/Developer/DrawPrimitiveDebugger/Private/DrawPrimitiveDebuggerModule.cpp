// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Modules/ModuleManager.h"
#include "DrawPrimitiveDebugger.h"
#include "SDrawPrimitiveDebugger.h"
#include "ViewDebug.h"
#include "Widgets/Docking/SDockTab.h"

DEFINE_LOG_CATEGORY(LogDrawPrimitiveDebugger)

#if WITH_PRIMITIVE_DEBUGGER
static FAutoConsoleCommand SummonDebuggerCmd(
		TEXT("PrimitiveDebugger.Open"),
		TEXT("Summons the primitive debugger window."),
		FConsoleCommandDelegate::CreateLambda([]() { IDrawPrimitiveDebugger::Get().OpenDebugWindow(); })
		);
/*static FAutoConsoleCommand EnableLiveCaptureCmd(
		TEXT("PrimitiveDebugger.EnableLiveCapture"),
		TEXT("Enables live capture for the primitive debugger."),
		FConsoleCommandDelegate::CreateLambda([]() { IDrawPrimitiveDebugger::Get().EnableLiveCapture(); })
		);
static FAutoConsoleCommand DisableLiveCaptureCmd(
		TEXT("PrimitiveDebugger.DisableLiveCapture"),
		TEXT("Disables live capture for the primitive debugger."),
		FConsoleCommandDelegate::CreateLambda([]() { IDrawPrimitiveDebugger::Get().DisableLiveCapture(); })
		);*/ // TODO: Re-enable these commands once live capture performance has been fixed
static FAutoConsoleCommand TakeSnapshotCmd(
		TEXT("PrimitiveDebugger.Snapshot"),
		TEXT("Captures the primitives rendered on the next frame for the primitive debugger."),
		FConsoleCommandDelegate::CreateLambda([]() { IDrawPrimitiveDebugger::Get().CaptureSingleFrame(); })
		);
#endif

class FDrawPrimitiveDebuggerModule : public IDrawPrimitiveDebugger
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual void CaptureSingleFrame() override;
	virtual bool IsLiveCaptureEnabled() const override;
	virtual void EnableLiveCapture() override;
	virtual void DisableLiveCapture() override;
	virtual void DiscardCaptureData() override;
	virtual void OpenDebugWindow() override;
	virtual void CloseDebugWindow() override;
#if WITH_PRIMITIVE_DEBUGGER
	static const FViewDebugInfo& GetViewDebugInfo();
#endif

private:
	bool bLiveCaptureEnabled = false;
	FDelegateHandle UpdateDelegateHandle;
#if WITH_PRIMITIVE_DEBUGGER
	TSharedPtr<SDrawPrimitiveDebugger> DebuggerWidget;
	TSharedPtr<SDockTab> DebuggerTab;
	FDelegateHandle OnWorldDestroyedHandle;
	FDelegateHandle OnWorldAddedHandle;

	TSharedRef<SDockTab> MakeDrawPrimitiveDebuggerTab(const FSpawnTabArgs&);

	void OnTabClosed(TSharedRef<SDockTab> Tab);
	
	void OnUpdateViewInformation();

	void HandleWorldDestroyed(UWorld* World);
	void HandleWorldAdded(UWorld* World);
#endif
};

IMPLEMENT_MODULE(FDrawPrimitiveDebuggerModule, DrawPrimitiveDebugger)

void FDrawPrimitiveDebuggerModule::StartupModule()
{
	bLiveCaptureEnabled = false;
#if WITH_PRIMITIVE_DEBUGGER
	UpdateDelegateHandle = FViewDebugInfo::Instance.AddUpdateHandler(this, &FDrawPrimitiveDebuggerModule::OnUpdateViewInformation);
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner("Primitive Debugger", FOnSpawnTab::CreateRaw(this, &FDrawPrimitiveDebuggerModule::MakeDrawPrimitiveDebuggerTab) );
#endif
}

void FDrawPrimitiveDebuggerModule::ShutdownModule()
{
#if WITH_PRIMITIVE_DEBUGGER
	FViewDebugInfo::Instance.RemoveUpdateHandler(UpdateDelegateHandle);
#endif
}

void FDrawPrimitiveDebuggerModule::CaptureSingleFrame()
{
#if WITH_PRIMITIVE_DEBUGGER
	UE_LOGF(LogDrawPrimitiveDebugger, Log, "Collecting a single frame graphics data capture");
	FViewDebugInfo::Instance.CaptureNextFrame();
#endif
}

bool FDrawPrimitiveDebuggerModule::IsLiveCaptureEnabled() const
{
	return bLiveCaptureEnabled;
}

void FDrawPrimitiveDebuggerModule::EnableLiveCapture()
{
#if WITH_PRIMITIVE_DEBUGGER
	if (!bLiveCaptureEnabled)
	{
		UE_LOGF(LogDrawPrimitiveDebugger, Log, "Enabling live graphics data capture");
		bLiveCaptureEnabled = true;
		FViewDebugInfo::Instance.EnableLiveCapture();
	}
#endif
}

void FDrawPrimitiveDebuggerModule::DisableLiveCapture()
{
#if WITH_PRIMITIVE_DEBUGGER
	if (bLiveCaptureEnabled)
	{
		UE_LOGF(LogDrawPrimitiveDebugger, Log, "Disabling live graphics data capture");
		bLiveCaptureEnabled = false;
		FViewDebugInfo::Instance.DisableLiveCapture();
	}
#endif
}

void FDrawPrimitiveDebuggerModule::DiscardCaptureData()
{
#if WITH_PRIMITIVE_DEBUGGER
	UE_LOGF(LogDrawPrimitiveDebugger, Log, "Clearing the captured graphics data from the Primitive Debugger");
	FViewDebugInfo::Instance.ClearCaptureData();
#endif
}

#if WITH_PRIMITIVE_DEBUGGER
const FViewDebugInfo& FDrawPrimitiveDebuggerModule::GetViewDebugInfo()
{
	return FViewDebugInfo::Get();
}
#endif

void FDrawPrimitiveDebuggerModule::OpenDebugWindow()
{
#if WITH_PRIMITIVE_DEBUGGER
	if (!DebuggerTab.IsValid())
	{
		UE_LOGF(LogDrawPrimitiveDebugger, Log, "Opening the Primitive Debugger");
		OnWorldDestroyedHandle = GEngine->OnWorldDestroyed().AddRaw(this, &FDrawPrimitiveDebuggerModule::HandleWorldDestroyed);
		OnWorldAddedHandle = FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FDrawPrimitiveDebuggerModule::HandleWorldAdded);
		if (!FViewDebugInfo::Instance.HasEverUpdated())
		{
			CaptureSingleFrame();
		}
		FGlobalTabmanager::Get()->TryInvokeTab(FTabId("Primitive Debugger"));
	}
#endif
}

void FDrawPrimitiveDebuggerModule::CloseDebugWindow()
{
#if WITH_PRIMITIVE_DEBUGGER
	if (DebuggerTab.IsValid())
	{
		UE_LOGF(LogDrawPrimitiveDebugger, Log, "Closing the Primitive Debugger");
		DebuggerTab->RequestCloseTab();
		GEngine->OnWorldDestroyed().Remove(OnWorldDestroyedHandle);
		FCoreUObjectDelegates::PostLoadMapWithWorld.Remove(OnWorldAddedHandle);
	}
#endif
}

#if WITH_PRIMITIVE_DEBUGGER
TSharedRef<SDockTab> FDrawPrimitiveDebuggerModule::MakeDrawPrimitiveDebuggerTab(const FSpawnTabArgs&)
{
	DebuggerTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.OnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FDrawPrimitiveDebuggerModule::OnTabClosed));
	
	if (!DebuggerWidget.IsValid())
	{
		DebuggerWidget = SNew(SDrawPrimitiveDebugger);
		DebuggerWidget->SetActiveWorld(GEngine->GetCurrentPlayWorld());
		DebuggerTab->SetContent(DebuggerWidget.ToSharedRef());
	}
	return DebuggerTab.ToSharedRef();
}

void FDrawPrimitiveDebuggerModule::OnTabClosed(TSharedRef<SDockTab> Tab)
{
	DebuggerTab.Reset();
	DebuggerTab = nullptr;
	DebuggerWidget.Reset();
	DebuggerWidget = nullptr;
}

void FDrawPrimitiveDebuggerModule::OnUpdateViewInformation()
{
	if (DebuggerWidget && DebuggerWidget.IsValid())
	{
		DebuggerWidget->Refresh();
	}
}

void FDrawPrimitiveDebuggerModule::HandleWorldDestroyed(UWorld* World)
{
#if WITH_EDITOR
	if (IsValid(World) && !World->IsGameWorld())
	{
		// We should only have to care about the game world, not any editor specific worlds
		return;
	}
#endif
	if (DebuggerWidget.IsValid())
	{
		// Clear all data bound to the debugger so that it no longer attempts to access them
		DebuggerWidget->ClearAllEntries();
		
		DebuggerWidget->SetActiveWorld(nullptr);
	}
	DiscardCaptureData();
}

void FDrawPrimitiveDebuggerModule::HandleWorldAdded(UWorld* World)
{
#if WITH_EDITOR
	if (IsValid(World) && !World->IsGameWorld())
	{
		// We should only have to care about the game world, not any editor specific worlds
		return;
	}
#endif
	if (DebuggerWidget.IsValid())
	{
		DebuggerWidget->SetActiveWorld(World);
	}
}

#endif