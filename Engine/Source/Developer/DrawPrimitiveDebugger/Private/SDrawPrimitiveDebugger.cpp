// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDrawPrimitiveDebugger.h"

#include "DrawPrimitiveDebugger.h"
#include "DrawPrimitiveDebuggerConfig.h"
#include "SlateOptMacros.h"
#include "PrimitiveSceneProxy.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "Components/LineBatchComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Framework/Application/SlateApplication.h"
#include "Components/PrimitiveComponent.h"
#include "Fonts/FontMeasure.h"
#include "Materials/Material.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PRIMITIVE_DEBUGGER"

#if WITH_PRIMITIVE_DEBUGGER

#define PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS UE_ENABLE_DEBUG_DRAWING

DECLARE_STATS_GROUP(TEXT("PrimitiveDebugger"), STATGROUP_PrimitiveDebugger, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("Primitive Debugger - Process Primitives Refresh"), STAT_PrimitiveDebuggerRefresh, STATGROUP_PrimitiveDebugger);
DECLARE_CYCLE_STAT(TEXT("Primitive Debugger - Process Primitives Gather"), STAT_PrimitiveDebuggerRefreshGather, STATGROUP_PrimitiveDebugger);
DECLARE_CYCLE_STAT(TEXT("Primitive Debugger - Process Primitives Update Visible"), STAT_PrimitiveDebuggerUpdateVis, STATGROUP_PrimitiveDebugger);
DECLARE_CYCLE_STAT(TEXT("Primitive Debugger - UI Make Cell"), STAT_PrimitiveDebuggerMakeCell, STATGROUP_PrimitiveDebugger);
DECLARE_CYCLE_STAT(TEXT("Primitive Debugger - UI Make Cell: Visible"), STAT_PrimitiveDebuggerMakeCellVisible, STATGROUP_PrimitiveDebugger);
DECLARE_CYCLE_STAT(TEXT("Primitive Debugger - UI Make Cell: Pinned"), STAT_PrimitiveDebuggerMakeCellPinned, STATGROUP_PrimitiveDebugger);
DECLARE_CYCLE_STAT(TEXT("Primitive Debugger - UI Make Cell: Name"), STAT_PrimitiveDebuggerMakeCellName, STATGROUP_PrimitiveDebugger);
DECLARE_CYCLE_STAT(TEXT("Primitive Debugger - UI Make Cell: ActorClass"), STAT_PrimitiveDebuggerMakeCellActorClass, STATGROUP_PrimitiveDebugger);
DECLARE_CYCLE_STAT(TEXT("Primitive Debugger - UI Make Cell: Actor"), STAT_PrimitiveDebuggerMakeCellActor, STATGROUP_PrimitiveDebugger);

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

SPrimitiveDebuggerDetailView::~SPrimitiveDebuggerDetailView() = default;

void SPrimitiveDebuggerDetailView::Construct(const FArguments& InArgs)
{
	static const FMargin Margin(5, 2, 5, 2);
	static const FMargin MarginInterior(5, 2, 0, 2);
	PrimitiveDebugger = InArgs._PrimitiveDebugger;
	ChildSlot
	[
		SNew(SScrollBox)
		.Orientation(Orient_Vertical)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always)
		+ SScrollBox::Slot()
		[
			SAssignNew(DetailPropertiesWidget, SVerticalBox)
		]
	];
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_NameLabel", "Name:"), &SPrimitiveDebuggerDetailView::GetSelectedPrimitiveName, nullptr,
		&SPrimitiveDebuggerDetailView::GetSelectedPrimitiveName, /* bSupportHighlighting */ true);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_TypeLabel", "Type:"), &SPrimitiveDebuggerDetailView::GetSelectedPrimitiveType, nullptr,
		&SPrimitiveDebuggerDetailView::GetSelectedPrimitiveType, /* bSupportHighlighting */ true);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_ActorLabel", "Actor:"), &SPrimitiveDebuggerDetailView::GetSelectedActorName, nullptr,
		&SPrimitiveDebuggerDetailView::GetSelectedActorToolTip, /* bSupportHighlighting */ true);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_ActorClassLabel", "Actor Class:"), &SPrimitiveDebuggerDetailView::GetSelectedActorClassName, nullptr,
		&SPrimitiveDebuggerDetailView::GetSelectedActorClassToolTip, /* bSupportHighlighting */ true);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_LocationLabel", "Location:"), &SPrimitiveDebuggerDetailView::GetSelectedLocation);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_NaniteSupportLabel", "Supports Nanite:"), &SPrimitiveDebuggerDetailView::GetSelectedPrimitiveSupportsNanite,
		&SPrimitiveDebuggerDetailView::StaticMeshDataVisibility);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_NaniteEnabledLabel", "Nanite Enabled:"), &SPrimitiveDebuggerDetailView::GetSelectedPrimitiveNaniteEnabled,
		&SPrimitiveDebuggerDetailView::StaticMeshDataVisibility);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_CurrentLODLabel", "Current LOD:"), &SPrimitiveDebuggerDetailView::GetSelectedLOD,
		&SPrimitiveDebuggerDetailView::NonNaniteDataVisibility);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_AvailableLODsLabel", "Available LODs:"), &SPrimitiveDebuggerDetailView::GetSelectedNumLODs,
		&SPrimitiveDebuggerDetailView::NonNaniteDataVisibility);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_DrawCallsLabel", "Draw Calls:"), &SPrimitiveDebuggerDetailView::GetSelectedDrawCallCount,
		&SPrimitiveDebuggerDetailView::NonNaniteDataVisibility);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_TrianglesLabel", "Triangles:"), &SPrimitiveDebuggerDetailView::GetSelectedTriangleCount,
		&SPrimitiveDebuggerDetailView::NonNaniteDataVisibility);
	GenerateDetailPanelEntry(LOCTEXT("DetailPanel_BonesLabel", "Bones:"), &SPrimitiveDebuggerDetailView::GetSelectedBoneCount,
		&SPrimitiveDebuggerDetailView::SkeletalMeshDataVisibility);
	
	DetailPropertiesWidget->AddSlot()
	.Padding(Margin)
	.AutoHeight()
	[
		SNew(SExpandableArea)
		.Padding(MarginInterior)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DetailPanel_MaterialsLabel", "Materials"))
		]
		.BodyContent()
		[
			GetSelectedMaterialsWidget()
		]
	];
	
	DetailPropertiesWidget->AddSlot()
	.Padding(Margin)
	.AutoHeight()
	[
		SNew(SExpandableArea)
		.Padding(MarginInterior)
		.InitiallyCollapsed(false)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DetailPanel_AdvancedOptionsLabel", "Advanced"))
		]
		.BodyContent()
		[
			GetAdvancedOptionsWidget()
		]
	];
}

void SPrimitiveDebuggerDetailView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime,
	const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	if (!PrimitiveDebugger.IsValid())
	{
		return;
	}
	const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
	const FPrimitiveRowDataPtr Selection = DebuggerInstance->GetCurrentSelection();
	if (Selection.IsValid() && Selection->Owner.IsValid())
	{
		CurrentLOD = Selection->GetCurrentLOD(PlayerIndex, ViewIndex);
		bSelectionIsNaniteEnabledThisFrame = false;
		if (bSelectionSupportsNanite && Selection->IsPrimitiveValid())
		{
			const FPrimitiveSceneProxy* Proxy = Selection->ComponentInterface->GetSceneProxy();
			bSelectionIsNaniteEnabledThisFrame = Proxy && Proxy->IsNaniteMesh();
		}
	}
	else
	{
		CurrentLOD = nullptr;
		bSelectionIsNaniteEnabledThisFrame = false;
	}
}

static const FText PlaceholderValue = FText::FromString(TEXT("-"));
static const FText TrueTextValue = LOCTEXT("DetailPanel_True", "true");
static const FText FalseTextValue = LOCTEXT("DetailPanel_False", "false");
static const FText InvalidTextValue = LOCTEXT("DetailPanel_InvalidValue", "INVALID");

void SPrimitiveDebuggerDetailView::UpdateSelection()
{
	CurrentLOD = nullptr;
	SelectedActorName = PlaceholderValue;
	SelectedActorPath = PlaceholderValue;
	SelectedActorClassName = PlaceholderValue;
	SelectedActorClassPath = PlaceholderValue;
	SelectedPrimitiveType = PlaceholderValue;
	bSelectionSupportsNanite = false;
	SelectedComponentType = nullptr;
	SelectedAsStaticMesh = nullptr;
	SelectedAsSkinnedMesh = nullptr;
	if (PrimitiveDebugger.IsValid())
	{
		const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
		const FPrimitiveRowDataPtr Selection = DebuggerInstance->GetCurrentSelection();
		if (Selection.IsValid() && Selection->Owner.IsValid())
		{
			CurrentLOD = Selection->GetCurrentLOD(PlayerIndex, ViewIndex);
			SelectedActorName = FText::FromString(Selection->GetOwnerName());
			SelectedActorPath = FText::FromString(Selection->Owner->GetPathName());
			SelectedActorClassName = FText::FromString(Selection->Owner->GetClass()->GetName());
			SelectedActorClassPath = FText::FromString(Selection->Owner->GetClass()->GetPathName());
			if (Selection->IsPrimitiveValid())
			{
				UObject* Component = Selection->ComponentInterface->GetUObject();
				SelectedComponentType = Component->GetClass();
				SelectedAsStaticMesh = Cast<UStaticMeshComponent>(Component);
				SelectedAsSkinnedMesh = Cast<USkinnedMeshComponent>(Component);
				SelectedPrimitiveType = FText::FromString(SelectedComponentType->GetName());
				if (SelectedAsStaticMesh.IsValid())
				{
					bSelectionSupportsNanite = SelectedAsStaticMesh->GetStaticMesh()->HasValidNaniteData();
					// TODO: Handle support for non-static mesh nanite primitives as they become available
				}
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
				if (SelectedAsSkinnedMesh.IsValid() && DebuggerInstance->IsEntryShowingDebugBones(Selection->ComponentId) && SelectedAsSkinnedMesh->ShouldDrawDebugSkeleton())
				{
					SelectedAsSkinnedMesh->SetDebugDrawColor(FLinearColor(FColor::Yellow));
					SelectedAsSkinnedMesh->MarkRenderStateDirty();
				}
#endif
				const FPrimitiveSceneProxy* Proxy = Selection->ComponentInterface->GetSceneProxy();
				bSelectionIsNaniteEnabledThisFrame = Proxy && Proxy->IsNaniteMesh();
			}
		}
	}
	GetSelectedMaterialsWidget();
	GetAdvancedOptionsWidget();
}

void SPrimitiveDebuggerDetailView::ReleaseSelection()
{
	if (!PrimitiveDebugger.IsValid())
	{
		return;
	}
	const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
	const FPrimitiveRowDataPtr Selection = DebuggerInstance->GetCurrentSelection();
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	if (Selection.IsValid())
	{
		if (SelectedAsSkinnedMesh.IsValid() && DebuggerInstance->IsEntryShowingDebugBones(Selection->ComponentId) && SelectedAsSkinnedMesh->ShouldDrawDebugSkeleton())
		{
			SelectedAsSkinnedMesh->SetDebugDrawColor(FLinearColor(FColor::Orange));
			SelectedAsSkinnedMesh->MarkRenderStateDirty();
		}
	}
#endif
}

FText SPrimitiveDebuggerDetailView::GetSelectedPrimitiveName() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	const FPrimitiveRowDataPtr Selection = PrimitiveDebugger.Pin()->GetCurrentSelection();
	return Selection.IsValid() ? FText::FromString(Selection->Name) : PlaceholderValue;
}

FText SPrimitiveDebuggerDetailView::GetSelectedPrimitiveType() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	return SelectedPrimitiveType;
}

FText SPrimitiveDebuggerDetailView::GetSelectedActorName() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	return SelectedActorName;
}

FText SPrimitiveDebuggerDetailView::GetSelectedActorToolTip() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	return SelectedActorPath;
}

FText SPrimitiveDebuggerDetailView::GetSelectedActorClassName() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	return SelectedActorClassName;
}

FText SPrimitiveDebuggerDetailView::GetSelectedActorClassToolTip() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	return SelectedActorClassPath;
}

FText SPrimitiveDebuggerDetailView::GetSelectedPrimitiveNaniteEnabled() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	return bSelectionIsNaniteEnabledThisFrame ? TrueTextValue : FalseTextValue;
}

FText SPrimitiveDebuggerDetailView::GetSelectedPrimitiveSupportsNanite() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	return bSelectionSupportsNanite ? TrueTextValue : FalseTextValue;
}

FText SPrimitiveDebuggerDetailView::GetSelectedDrawCallCount() const
{
	if (!CurrentLOD)
	{
		return PlaceholderValue;
	}
	return FText::FromString(FString::FromInt(CurrentLOD->GetDrawCount()));
}

FText SPrimitiveDebuggerDetailView::GetSelectedLocation() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	const FPrimitiveRowDataPtr Selection = PrimitiveDebugger.Pin()->GetCurrentSelection();
	return Selection.IsValid() && Selection->IsPrimitiveValid() ?
		FText::FromString(Selection->GetPrimitiveLocation().ToString()) :
		PlaceholderValue;
}

FText SPrimitiveDebuggerDetailView::GetSelectedLOD() const
{
	TOptional<int> LOD = GetSelectedLODValue();
	return LOD.IsSet() && LOD.GetValue() >= 0 ? FText::FromString(FString::FromInt(LOD.GetValue())) : PlaceholderValue;
}

FText SPrimitiveDebuggerDetailView::GetSelectedNumLODs() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	const FPrimitiveRowDataPtr Selection = PrimitiveDebugger.Pin()->GetCurrentSelection();
	return Selection.IsValid() ?
		FText::FromString(FString::FromInt(Selection->GetNumLODs())) :
		PlaceholderValue;
}

TOptional<int> SPrimitiveDebuggerDetailView::GetSelectedLODValue() const
{
	if (!CurrentLOD)
	{
		return TOptional<int>();
	}
	return CurrentLOD->LODIndex;
}

TOptional<int> SPrimitiveDebuggerDetailView::GetSelectedForcedLODValue() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return TOptional<int>();
	}
	const FPrimitiveRowDataPtr Selection = PrimitiveDebugger.Pin()->GetCurrentSelection();
	if (Selection.IsValid())
	{
		int ForcedLOD = 0;
		if (SelectedAsStaticMesh.IsValid())
		{
			ForcedLOD = SelectedAsStaticMesh->ForcedLodModel;
		}
		if (SelectedAsSkinnedMesh.IsValid())
		{
			ForcedLOD = SelectedAsSkinnedMesh->GetForcedLOD();
		}
		if (ForcedLOD > 0)
		{
			return FMath::Clamp(ForcedLOD - 1, 0, GetSelectedNumLODsValue().Get(0) - 1);
		}
	}
	return TOptional<int>();
}

TOptional<int> SPrimitiveDebuggerDetailView::GetSelectedNumLODsValue() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return 0;
	}
	const FPrimitiveRowDataPtr Selection = PrimitiveDebugger.Pin()->GetCurrentSelection();
	if (Selection.IsValid())
	{
		int NumLODs = Selection->GetNumLODs();
		return NumLODs;
	}
	return 0;
}

TOptional<int> SPrimitiveDebuggerDetailView::GetSelectedForcedLODSliderMaxValue() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return 0;
	}
	const FPrimitiveRowDataPtr Selection = PrimitiveDebugger.Pin()->GetCurrentSelection();
	if (Selection.IsValid())
	{
		return Selection->GetNumLODs() - 1;
	}
	return 0;
}

FText SPrimitiveDebuggerDetailView::GetSelectedTriangleCount() const
{
	if (!CurrentLOD)
	{
		return PlaceholderValue;
	}
	return FText::FromString(FString::FromInt(CurrentLOD->Triangles));
}

FText SPrimitiveDebuggerDetailView::GetSelectedBoneCount() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return FText::GetEmpty();
	}
	return SelectedAsSkinnedMesh.IsValid() ?
		FText::FromString(FString::FromInt(SelectedAsSkinnedMesh->GetNumBones())) :
		PlaceholderValue;
}

void SPrimitiveDebuggerDetailView::GenerateDetailPanelEntry(const FText& Label,
	FText(SPrimitiveDebuggerDetailView::* ValueGetter)() const,
	EVisibility (SPrimitiveDebuggerDetailView::* VisibilityGetter)() const,
	FText(SPrimitiveDebuggerDetailView::* TooltipGetter)() const,
	bool bSupportHighlighting) const
{
	static const FMargin Margin(5, 2, 5, 2);
	static constexpr int32 LabelColumnWidth = 1;
	static constexpr int32 ValueColumnWidth = 2;
	
	TSharedRef<STextBlock> EntryValue = SNew(STextBlock)
		.Text(this, ValueGetter)
		.Justification(ETextJustify::Left)
		.OverflowPolicy(ETextOverflowPolicy::Ellipsis);

	if (TooltipGetter)
	{
		TAttribute<FText> TooltipTextAttribute = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, TooltipGetter));
		EntryValue->SetToolTipText(TooltipTextAttribute);
	}
	if (bSupportHighlighting)
	{
		TAttribute<FText> HighlightTextAttribute = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(PrimitiveDebugger.Pin().Get(), &SDrawPrimitiveDebugger::GetFilterText));
		EntryValue->SetHighlightText(HighlightTextAttribute);
	}
	
	TSharedRef<SHorizontalBox> Entry = SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.FillWidth(LabelColumnWidth)
		[
			SNew(STextBlock)
			.Text(Label)
			.Justification(ETextJustify::Left)
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.FillWidth(ValueColumnWidth)
		[
			EntryValue
		];

	if (VisibilityGetter)
	{
		TAttribute<EVisibility> EntryVisibilityAttribute = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, VisibilityGetter));
		Entry->SetVisibility(EntryVisibilityAttribute);
	}

	DetailPropertiesWidget->AddSlot()
	.Padding(Margin)
	.AutoHeight()
	[
		Entry
	];
}

TSharedRef<SVerticalBox> SPrimitiveDebuggerDetailView::GetSelectedMaterialsWidget()
{
	if (!MaterialsWidget.IsValid()) SAssignNew(MaterialsWidget, SVerticalBox);
	else
	{
		MaterialsWidget->ClearChildren();
	}
	
	if (!PrimitiveDebugger.IsValid())
	{
		return MaterialsWidget.ToSharedRef();
	}
	
	const FPrimitiveRowDataPtr Selection = PrimitiveDebugger.Pin()->GetCurrentSelection();
	if (Selection.IsValid())
	{
		if (CurrentLOD)
		{
			const int32 Count = CurrentLOD->MaterialIndices.Num();

			for (int i = 0; i < Count; i++)
			{
				CreateMaterialEntry(Selection->GetMaterial(CurrentLOD->MaterialIndices[i]), i, false);
			}
			if (Selection->OverlayMaterial.IsValid())
			{
				CreateMaterialEntry(Selection->OverlayMaterial.Get(), -1, true);
			}
		}
		else if (bSelectionIsNaniteEnabledThisFrame)
		{
			const int32 Count = Selection->Materials.Num();
			for (int i = 0; i < Count; i++)
			{
				CreateMaterialEntry(Selection->Materials[i].Get(), i, false);
			}
			if (Selection->OverlayMaterial.IsValid())
			{
				CreateMaterialEntry(Selection->OverlayMaterial.Get(), -1, true);
			}
		}
	}
	return MaterialsWidget.ToSharedRef();
}

void SPrimitiveDebuggerDetailView::CreateMaterialEntry(const UMaterialInterface* MI, int Index, bool bIsOverlay)
{
	static const FMargin Margin(5, 2, 5, 2);
	static const FMargin MarginInterior(10, 2, 0, 2);
	static constexpr int32 LabelColumnWidth = 1;
	static constexpr int32 ValueColumnWidth = 2;

	FString MaterialName = "NULL";
	FString MaterialPath = "NULL";
	const TSharedRef<SVerticalBox> TextureList = SNew(SVerticalBox);
	
	if (MI && MI->GetMaterial())
	{
		MaterialName = MI->GetMaterial()->GetName();
		MaterialPath = MI->GetMaterial()->GetPathName().LeftChop(MaterialName.Len() + 1);
		
		TArray<UTexture*> Textures;
		MI->GetUsedTextures(Textures, GetCurrentMaterialQualityLevelChecked(), GetMaxShaderPlatformChecked());
		const int32 TexCount = Textures.Num();
		for (int t = 0; t < TexCount; t++)
		{
			if (IsValid(Textures[t]))
			{
				FString TextureName = Textures[t]->GetName();
				
				TextureList->AddSlot()
				.Padding(Margin)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text(FText::FromString(TextureName))
						.Justification(ETextJustify::Right)
						.ToolTipText(FText::FromString(Textures[t]->GetPathName().LeftChop(TextureName.Len() + 1)))
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.HighlightText(PrimitiveDebugger.Pin().Get(), &SDrawPrimitiveDebugger::GetFilterText)
					]
				];
			}
			else
			{
				TextureList->AddSlot()
				.Padding(Margin)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(InvalidTextValue)
					.Justification(ETextJustify::Left)
				];
			}
		}
	}
	MaterialsWidget->AddSlot()
	.Padding(MarginInterior)
	.AutoHeight()
	[
		SNew(SExpandableArea)
		.Padding(MarginInterior)
		.InitiallyCollapsed(true)
		.HeaderContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(LabelColumnWidth)
			[
				SNew(STextBlock)
				.Text(bIsOverlay ? LOCTEXT("DetailPanel_OverlayLabel", "Overlay") : FText::FromString(FString::FromInt(Index)))
				.Justification(ETextJustify::Left)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.FillWidth(ValueColumnWidth)
			[
				SNew(STextBlock)
				.Text(FText::FromString(MaterialName))
				.Justification(ETextJustify::Left)
				.ToolTipText(FText::FromString(MaterialPath))
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				.HighlightText(PrimitiveDebugger.Pin().Get(), &SDrawPrimitiveDebugger::GetFilterText)
			]
		]
		.BodyContent()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(MarginInterior)
			.AutoHeight()
			[
				SNew(SExpandableArea)
				.Padding(MarginInterior)
				.InitiallyCollapsed(true)
				.HeaderContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DetailPanel_TexturesLabel", "Textures"))
				]
				.BodyContent()
				[
					TextureList
				]
			]
		]
	];
}

TSharedRef<SVerticalBox> SPrimitiveDebuggerDetailView::GetAdvancedOptionsWidget()
{
	static const FMargin Margin(5, 2, 5, 2);
	static constexpr int32 LabelColumnWidth = 1;
	static constexpr int32 ValueColumnWidth = 2;
	
	if (!AdvancedOptionsWidget.IsValid()) SAssignNew(AdvancedOptionsWidget, SVerticalBox);
	else
	{
		AdvancedOptionsWidget->ClearChildren();
	}
	
	if (!PrimitiveDebugger.IsValid())
	{
		return AdvancedOptionsWidget.ToSharedRef();
	}
	
	const FPrimitiveRowDataPtr Selection = PrimitiveDebugger.Pin()->GetCurrentSelection();
	if (Selection.IsValid() && Selection->IsPrimitiveValid())
	{
		const FText ShowBoundsTooltip = LOCTEXT("DetailPanel_ShowBoundsTooltip", "Should a debug box of this mesh's bounds be displayed? DEVELOPMENT BUILDS ONLY");
		const FText ShowBonesTooltip = LOCTEXT("DetailPanel_ShowBonesTooltip", "Should a debug display of this mesh's skeleton be displayed? DEVELOPMENT BUILDS ONLY");
		const FText ForcedLODTooltip = LOCTEXT("DetailPanel_ForcedLODTooltip", "Should a specific LOD level be forced on this primitive?.");
		const FText ForcedLODIndexTooltip = LOCTEXT("DetailPanel_ForcedLODIndexTooltip", "Controls the forced LOD level of this primitive.");
		const FText ForceDisableNaniteTooltip = LOCTEXT("DetailPanel_ForcedDisableNaniteTooltip", "Should nanite be force disabled on this component?");
		
		AdvancedOptionsWidget->AddSlot()
		.Padding(Margin)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
#if !PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
			.IsEnabled(false)
#endif
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(LabelColumnWidth)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DetailPanel_ShowBoundsLabel", "Show Bounds"))
				.Justification(ETextJustify::Left)
				.ToolTipText(ShowBoundsTooltip)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.FillWidth(ValueColumnWidth)
			[
				SNew(SCheckBox)
				.IsChecked(this, &SPrimitiveDebuggerDetailView::ShowDebugBoundsState)
				.OnCheckStateChanged(this, &SPrimitiveDebuggerDetailView::OnToggleDebugBounds)
				.ToolTipText(ShowBoundsTooltip)
			]
		];
		AdvancedOptionsWidget->AddSlot()
		.Padding(Margin)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPrimitiveDebuggerDetailView::SkeletalMeshDataVisibility)
#if !PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
			.IsEnabled(false)
#endif
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(LabelColumnWidth)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DetailPanel_ShowBonesLabel", "Show Bones"))
				.Justification(ETextJustify::Left)
				.ToolTipText(ShowBonesTooltip)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.FillWidth(ValueColumnWidth)
			[
				SNew(SCheckBox)
				.IsChecked(this, &SPrimitiveDebuggerDetailView::ShowDebugBonesState)
				.OnCheckStateChanged(this, &SPrimitiveDebuggerDetailView::OnToggleDebugBones)
				.ToolTipText(ShowBonesTooltip)
			]
		];
		AdvancedOptionsWidget->AddSlot()
		.Padding(Margin)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPrimitiveDebuggerDetailView::OptionVisibilityForceLOD)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(LabelColumnWidth)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DetailPanel_ForceLODLabel", "Force LOD"))
				.Justification(ETextJustify::Left)
				.ToolTipText(ForcedLODTooltip)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.FillWidth(ValueColumnWidth)
			[
				SNew(SCheckBox)
				.IsChecked(this, &SPrimitiveDebuggerDetailView::ForceLODState)
				.OnCheckStateChanged(this, &SPrimitiveDebuggerDetailView::OnToggleForceLOD)
				.ToolTipText(ForcedLODTooltip)
			]
		];
		AdvancedOptionsWidget->AddSlot()
		.Padding(Margin)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPrimitiveDebuggerDetailView::OptionVisibilityForceLOD)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(LabelColumnWidth * 2)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DetailPanel_ForcedLODIndexLabel", "Forced LOD Index"))
				.Justification(ETextJustify::Left)
				.ToolTipText(ForcedLODIndexTooltip)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.FillWidth(ValueColumnWidth / 2)
			[
				SNew(SNumericEntryBox<int>)
				.Value(this, &SPrimitiveDebuggerDetailView::GetSelectedForcedLODValue)
				.MinValue(0)
				.MaxValue(this, &SPrimitiveDebuggerDetailView::GetSelectedForcedLODSliderMaxValue)
				.MinSliderValue(0)
				.MaxSliderValue(this, &SPrimitiveDebuggerDetailView::GetSelectedForcedLODSliderMaxValue)
				.Delta(1)
				.AllowSpin(true)
				.AllowWheel(true)
				.WheelStep(1)
				.UndeterminedString(LOCTEXT("DetailPanel_AutomaticLODPlaceholder", "Auto"))
				.IsEnabled(this, &SPrimitiveDebuggerDetailView::IsForceLODIndexSliderEnabled)
				.OnValueChanged(this, &SPrimitiveDebuggerDetailView::HandleForceLOD)
				.ToolTipText(ForcedLODIndexTooltip)
				.Justification(ETextJustify::Right)
			]
		];
		AdvancedOptionsWidget->AddSlot()
		.Padding(Margin)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPrimitiveDebuggerDetailView::OptionVisibilityForceDisableNanite)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(LabelColumnWidth)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DetailPanel_ForceDisableNaniteLabel", "Force Disable Nanite"))
				.Justification(ETextJustify::Left)
				.ToolTipText(ForceDisableNaniteTooltip)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.FillWidth(ValueColumnWidth)
			[
				SNew(SCheckBox)
				.IsChecked(this, &SPrimitiveDebuggerDetailView::ForceDisableNaniteState)
				.OnCheckStateChanged(this, &SPrimitiveDebuggerDetailView::OnToggleForceDisableNanite)
				.ToolTipText(ForceDisableNaniteTooltip)
			]
		];
	}
	return AdvancedOptionsWidget.ToSharedRef();
}

EVisibility SPrimitiveDebuggerDetailView::OptionVisibilityForceLOD() const
{
	TOptional<int> NumLODs = GetSelectedNumLODsValue();
	return !bSelectionIsNaniteEnabledThisFrame && NumLODs.IsSet() && NumLODs.GetValue() > 1 ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SPrimitiveDebuggerDetailView::OptionVisibilityForceDisableNanite() const
{
	return bSelectionSupportsNanite && SelectedAsStaticMesh.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState SPrimitiveDebuggerDetailView::ForceLODState() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return ECheckBoxState::Undetermined;
	}
	const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
	const FPrimitiveComponentId Selection = DebuggerInstance->GetCurrentSelectionId();
	return DebuggerInstance->DoesEntryHaveForcedLOD(Selection) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool SPrimitiveDebuggerDetailView::IsForceLODIndexSliderEnabled() const
{
	const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
	const FPrimitiveComponentId Selection = DebuggerInstance->GetCurrentSelectionId();
	return DebuggerInstance->DoesEntryHaveForcedLOD(Selection);
}

void SPrimitiveDebuggerDetailView::OnToggleForceLOD(ECheckBoxState state)
{
	if (!PrimitiveDebugger.IsValid())
	{
		return;
	}
	const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
	const FPrimitiveComponentId Selection = DebuggerInstance->GetCurrentSelectionId();
	if (state == ECheckBoxState::Unchecked)
	{
		DebuggerInstance->SetForcedLODForEntry(Selection, 0);
	}
	else if (state == ECheckBoxState::Checked)
	{
		DebuggerInstance->SetForcedLODForEntry(Selection, CurrentLOD->LODIndex + 1);
	}
}

ECheckBoxState SPrimitiveDebuggerDetailView::ForceDisableNaniteState() const
{
	if (SelectedAsStaticMesh.IsValid())
	{
		return SelectedAsStaticMesh->bForceDisableNanite ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void SPrimitiveDebuggerDetailView::OnToggleForceDisableNanite(ECheckBoxState state)
{
	if (!PrimitiveDebugger.IsValid())
	{
		return;
	}
	const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
	const FPrimitiveComponentId Selection = DebuggerInstance->GetCurrentSelectionId();
	if (state != ECheckBoxState::Undetermined)
	{
		DebuggerInstance->SetForceDisabledNaniteForEntry(Selection, state == ECheckBoxState::Checked);
	}
}

EVisibility SPrimitiveDebuggerDetailView::StaticMeshDataVisibility() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return EVisibility::Collapsed;
	}
	return SelectedAsStaticMesh.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SPrimitiveDebuggerDetailView::HandleForceLOD(int ForcedLOD)
{
	if (!PrimitiveDebugger.IsValid())
	{
		return;
	}
	const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
	const FPrimitiveComponentId Selection = DebuggerInstance->GetCurrentSelectionId();
	DebuggerInstance->SetForcedLODForEntry(Selection, ForcedLOD + 1);
}
	
ECheckBoxState SPrimitiveDebuggerDetailView::ShowDebugBoundsState() const
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	if (!PrimitiveDebugger.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}
	const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
	const FPrimitiveComponentId Selection = DebuggerInstance->GetCurrentSelectionId();
	return DebuggerInstance->IsEntryShowingDebugBounds(Selection) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
#else
	return ECheckBoxState::Unchecked;
#endif
}

void SPrimitiveDebuggerDetailView::OnToggleDebugBounds(ECheckBoxState state)
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	if (!PrimitiveDebugger.IsValid())
	{
		return;
	}
	const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
	const FPrimitiveComponentId Selection = DebuggerInstance->GetCurrentSelectionId();
	if (state != ECheckBoxState::Undetermined)
	{
		DebuggerInstance->SetShowDebugBoundsForEntry(Selection, state == ECheckBoxState::Checked);
	}
#endif
}

EVisibility SPrimitiveDebuggerDetailView::SkeletalMeshDataVisibility() const
{
	if (!PrimitiveDebugger.IsValid())
	{
		return EVisibility::Collapsed;
	}
	return SelectedAsSkinnedMesh.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState SPrimitiveDebuggerDetailView::ShowDebugBonesState() const
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	if (!PrimitiveDebugger.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}
	return SelectedAsSkinnedMesh.IsValid() && SelectedAsSkinnedMesh->ShouldDrawDebugSkeleton() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
#else
	return ECheckBoxState::Unchecked;
#endif
}

void SPrimitiveDebuggerDetailView::OnToggleDebugBones(ECheckBoxState state)
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	if (!PrimitiveDebugger.IsValid())
	{
		return;
	}
	const TSharedPtr<SDrawPrimitiveDebugger> DebuggerInstance = PrimitiveDebugger.Pin();
	const FPrimitiveComponentId Selection = DebuggerInstance->GetCurrentSelectionId();
	if (state != ECheckBoxState::Undetermined)
	{
		DebuggerInstance->SetShowDebugBonesForEntry(Selection, state == ECheckBoxState::Checked);
	}
#endif
}

EVisibility SPrimitiveDebuggerDetailView::NaniteDataVisibility() const
{
	return bSelectionSupportsNanite && bSelectionIsNaniteEnabledThisFrame ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SPrimitiveDebuggerDetailView::NonNaniteDataVisibility() const
{
	return !bSelectionSupportsNanite || !bSelectionIsNaniteEnabledThisFrame ? EVisibility::Visible : EVisibility::Collapsed;
}

SDrawPrimitiveDebugger::~SDrawPrimitiveDebugger()
{
	SetActiveWorld(nullptr);
}

void SDrawPrimitiveDebugger::Construct(const FArguments& InArgs)
{
	const TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(12.0f, 12.0f));

	ColumnHeader = SNew(SHeaderRow).ResizeMode(ESplitterResizeMode::Fill);
	const FName VisibilityColumn("Visible");
	const FName PinColumn("Pin");
	const FName NameColumn("Name");
	const FName ActorColumn("Actor");
	AddColumn(LOCTEXT("VisbleColumnLabel", "Visible"), VisibilityColumn);
	AddColumn(LOCTEXT("PinnedColumnLabel", "Pinned"), PinColumn);
	AddColumn(LOCTEXT("NameColumnLabel", "Name"), NameColumn);
	AddColumn(LOCTEXT("ActorColumnLabel", "Actor"), ActorColumn);

	FilterText = FText::GetEmpty();
	IDrawPrimitiveDebugger::Get().CaptureSingleFrame();

	Table = SNew(SListView<FPrimitiveRowDataPtr>)
		.ListItemsSource(&VisibleEntries)
		.HeaderRow(ColumnHeader)
		.OnGenerateRow(this, &SDrawPrimitiveDebugger::MakeRowWidget)
		.OnSelectionChanged(this, &SDrawPrimitiveDebugger::OnRowSelectionChanged)
		.ExternalScrollbar(VerticalScrollBar)
		.Orientation(Orient_Vertical)
		.ConsumeMouseWheel(EConsumeMouseWheel::Never)
		.SelectionMode(ESelectionMode::SingleToggle);
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(6, 6)
			.HAlign(HAlign_Fill)
			.FillWidth(2)
			[
				SAssignNew(SearchBox, SSearchBox)
				.InitialText(this, &SDrawPrimitiveDebugger::GetFilterText)
				.OnTextChanged(this, &SDrawPrimitiveDebugger::OnFilterTextChanged)
				.OnTextCommitted(this, &SDrawPrimitiveDebugger::OnFilterTextCommitted)
			]
			+ SHorizontalBox::Slot()
			.Padding(6, 6)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("RefreshButtonLabel", "Refresh"))
				.IsEnabled(this, &SDrawPrimitiveDebugger::CanCaptureSingleFrame)
				.OnClicked(this, &SDrawPrimitiveDebugger::OnRefreshClick)
			]
			+ SHorizontalBox::Slot()
			.Padding(6, 6)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("SaveToCSVButtonLabel", "Save to CSV"))
				.OnClicked(this, &SDrawPrimitiveDebugger::OnSaveClick)
			]
			/*+ SHorizontalBox::Slot()
			.Padding(6, 6)
			.AutoWidth()
			[
				SNew(SCheckBox)
				[
					SNew(STextBlock)
					.Text(FText::FromString("Enable Live Capture"))
					.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), UDrawPrimitiveDebuggerUserSettings::GetFontSize()))
				]
				.IsChecked(this, &SDrawPrimitiveDebugger::IsLiveCaptureChecked)
				.OnCheckStateChanged(this, &SDrawPrimitiveDebugger::OnToggleLiveCapture)
			]*/ // TODO: Re-enable after the performance issues have been fixed
		]
		+SVerticalBox::Slot()
		.Padding(6, 6)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(5)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Vertical)
				.ConsumeMouseWheel(EConsumeMouseWheel::Always)
				+SScrollBox::Slot()
				[
					Table.ToSharedRef()
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(5)
			[
				SAssignNew(DetailView, SPrimitiveDebuggerDetailView)
				.PrimitiveDebugger(SharedThis(this))
				.Visibility(this, &SDrawPrimitiveDebugger::DetailsPanelVisibility)
			]
		]
	];
}

void SDrawPrimitiveDebugger::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	RedrawAllDebugBounds();
}

FText SDrawPrimitiveDebugger::GetFilterText() const
{
	return FilterText;
}

void SDrawPrimitiveDebugger::OnFilterTextChanged(const FText& InFilterText)
{
	FilterText = InFilterText;
	UpdateVisibleRows();
	if (Table.IsValid())
	{
		Table->RequestListRefresh();
	}
}

void SDrawPrimitiveDebugger::OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnCleared)
	{
		SearchBox->SetText(FText::GetEmpty());
		OnFilterTextChanged(FText::GetEmpty());
	}
}

TSharedRef<ITableRow> SDrawPrimitiveDebugger::MakeRowWidget(FPrimitiveRowDataPtr InRowDataPtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SDrawPrimitiveDebuggerListViewRow, OwnerTable)
		.DrawPrimitiveDebugger(SharedThis(this))
		.RowDataPtr(InRowDataPtr);
}

void SDrawPrimitiveDebugger::UpdateVisibleRows()
{
	if (FilterText.IsEmptyOrWhitespace())
	{
		VisibleEntries = AvailableEntries;
	}
	else
	{
		VisibleEntries.Empty();

		const FString& ActiveFilterString = FilterText.ToString();
		for (const FPrimitiveRowDataPtr& RowData : AvailableEntries)
		{
			if (!RowData.IsValid() || !RowData->IsPrimitiveValid())
			{
				continue;
			}
			bool bPassesFilter = false;
			UObject* Component = RowData->ComponentInterface->GetUObject();
			
			if (RowData->Name.Contains(ActiveFilterString))
			{
				bPassesFilter = true;
			}
			else if (RowData->Owner.IsValid() &&
				(RowData->Owner->GetClass()->GetName().Contains(ActiveFilterString) ||
				RowData->Owner->GetFullName().Contains(ActiveFilterString)))
			{
				bPassesFilter = true;
			}
			else if (IsValid(Component->GetClass()) && Component->GetClass()->GetName().Contains(ActiveFilterString))
			{
				bPassesFilter = true;
			}
			else
			{
				for (const TWeakObjectPtr<UMaterialInterface> Material : RowData->Materials)
				{
					if (Material.IsValid() && IsValid(Material->GetMaterial()))
					{
						if (Material->GetMaterial()->GetName().Contains(ActiveFilterString))
						{
							bPassesFilter = true;
							break;
						}
						TArray<UTexture*> Textures;
						Material->GetUsedTextures(Textures, GetCurrentMaterialQualityLevelChecked(), GetMaxShaderPlatformChecked());
						const int32 TexCount = Textures.Num();
						for (int t = 0; t < TexCount; t++)
						{
							if (IsValid(Textures[t]) && Textures[t]->GetName().Contains(ActiveFilterString))
							{
								bPassesFilter = true;
								break;
							}
						}
						if (bPassesFilter) break;
					}
				}
			}

			if (bPassesFilter)
			{
				VisibleEntries.Add(RowData);
			}
		}
	}
	SortRows();
}

void SDrawPrimitiveDebugger::SortRows()
{
	VisibleEntries.Sort([this](FPrimitiveRowDataPtr A, FPrimitiveRowDataPtr B)
	{
		const bool bPinnedA = IsEntryPinned(A->ComponentId);
		const bool bPinnedB = IsEntryPinned(B->ComponentId);
		return (bPinnedA && !bPinnedB) || ((bPinnedA == bPinnedB) && *A < *B); // Put pinned entries first
	});
}

void SDrawPrimitiveDebugger::Refresh()
{
	SCOPE_CYCLE_COUNTER(STAT_PrimitiveDebuggerRefresh);
	OnRowSelectionChanged(nullptr, ESelectInfo::Direct);
	AvailableEntries.Empty();

	TSet<FPrimitiveComponentId> OutdatedEntries;
	// Get a list of all existing entry ids, any that are not rediscovered or marked for retention will be dropped
	Entries.GetKeys(OutdatedEntries);

	// Iterate over the new set of captured primitives to add new entries and check which entries to retain
	{
		SCOPE_CYCLE_COUNTER(STAT_PrimitiveDebuggerRefreshGather);
		FViewDebugInfo::Get().ForEachPrimitive([this, &OutdatedEntries](const FViewDebugInfo::FPrimitiveInfo& Primitive)
		{
			FPrimitiveDebuggerEntry* ExistingEntry = Entries.Find(Primitive.ComponentId);
			if (!ExistingEntry && Primitive.PrimitiveSceneInfo && Primitive.IsPrimitiveValid())
			{
				// Add the new entry
				FPrimitiveDebuggerEntry& NewEntry = Entries.Add(Primitive.ComponentId, FPrimitiveDebuggerEntry(Primitive));
				const UPrimitiveComponent* Component = Primitive.ComponentInterface->GetUObject<UPrimitiveComponent>();
				if (IsValid(Component) && !Component->GetVisibleFlag())
				{
					NewEntry.bHidden = true;
					NewEntry.bRetainDuringRefresh = true;
				}
				AvailableEntries.Add(NewEntry.Data);
			}
			else if (ExistingEntry)
			{
				// Get the latest version of the primitive data and make the entry available
				ExistingEntry->Data = MakeShared<const FViewDebugInfo::FPrimitiveInfo>(Primitive);
				AvailableEntries.Add(ExistingEntry->Data);
				OutdatedEntries.Remove(Primitive.ComponentId);
			}
		});
	}

	// Of any remaining old entries, add any marked with bRetainDuringRefresh to AvailableEntries and delete the rest
	for (const FPrimitiveComponentId& EntryId : OutdatedEntries)
	{
		const FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId);
		if (Entry && Entry->bRetainDuringRefresh && Entry->Data.IsValid() && Entry->Data->IsPrimitiveValid())
		{
			AvailableEntries.Add(Entry->Data);
		}
		else if (Entry)
		{
			FlushDebugVisualizationsForEntry(EntryId);
			Entries.Remove(EntryId);
		}
	}
	
	SCOPE_CYCLE_COUNTER(STAT_PrimitiveDebuggerUpdateVis);
	UpdateVisibleRows();
	if (Table.IsValid())
	{
		Table->RequestListRefresh();
	}
}

void SDrawPrimitiveDebugger::ClearAllEntries()
{
	DetailView->ReleaseSelection();
	Selection = nullptr;
	ResetDebuggerChanges();
	Entries.Empty();
	AvailableEntries.Empty();
	UpdateVisibleRows();
	if (Table.IsValid())
	{
		Table->RequestListRefresh();
	}
}

void SDrawPrimitiveDebugger::SetActiveWorld(UWorld* World)
{
	if (ActiveWorld == World)
	{
		return;
	}
	ResetDebuggerChanges();
	if (ActiveWorld.IsValid())
	{
		ActiveWorld->RemoveOnPreUnregisterAllActorComponentsHandler(ActorComponentsUnregisteredHandle);
	}
	if (IsValid(World))
	{
		ActorComponentsUnregisteredHandle = World->AddOnPreUnregisterAllActorComponentsHandler(FOnPreUnregisterAllActorComponents::FDelegate::CreateRaw(this, &SDrawPrimitiveDebugger::HandleActorCleanup));
	}
	ActiveWorld = World;
}

void SDrawPrimitiveDebugger::RemoveEntry(FPrimitiveRowDataPtr Entry)
{
	if (!Entry.IsValid())
	{
		return;
	}
	AvailableEntries.Remove(Entry);
	if (Selection && Selection->Data->ComponentId == Entry->ComponentId)
	{
		OnRowSelectionChanged(nullptr, ESelectInfo::Direct);
	}
	FlushDebugVisualizationsForEntry(Entry->ComponentId);
	Entries.Remove(Entry->ComponentId);
	VisibleEntries.Remove(Entry);
	if (Table.IsValid())
	{
		Table->RequestListRefresh();
	}
}

void SDrawPrimitiveDebugger::AddColumn(const FText& Name, const FName& ColumnId)
{
	const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FSlateFontInfo FontInfo = FSlateFontInfo(FCoreStyle::GetDefaultFont(), 12);
	const FName VisibilityColumn("Visible");
	const FName PinColumn("Pin");
	SHeaderRow::FColumn::FArguments& NewColumnArgs = SHeaderRow::Column(ColumnId)
			.DefaultLabel(Name);
	// Handle columns that can be narrow and fixed
	if (ColumnId.IsEqual(VisibilityColumn) || ColumnId.IsEqual(PinColumn))
	{
		NewColumnArgs = NewColumnArgs.FixedWidth(FontMeasure->Measure(Name, FontInfo).X);
	}
	ColumnHeader->AddColumn(NewColumnArgs);
}

void SDrawPrimitiveDebugger::OnChangeEntryVisibility(ECheckBoxState State, FPrimitiveRowDataPtr Data)
{
	if (!Data.IsValid() || !Data->IsPrimitiveValid())
	{
		return;
	}
	FPrimitiveDebuggerEntry* Entry = Data.IsValid() ? Entries.Find(Data->ComponentId) : nullptr;
	UPrimitiveComponent* Component = Data->ComponentInterface->GetUObject<UPrimitiveComponent>();
	if (Entry && IsValid(Component) && State != ECheckBoxState::Undetermined)
	{
		Component->SetVisibility(State == ECheckBoxState::Checked);
		if (State == ECheckBoxState::Unchecked)
		{
			Entry->bHidden = true;
			Entry->bRetainDuringRefresh = true;
		}
		else if (State == ECheckBoxState::Checked)
		{
			Entry->bHidden = false;
			Entry->bRetainDuringRefresh = false;
		}
	}
}

bool SDrawPrimitiveDebugger::IsEntryVisible(FPrimitiveComponentId EntryId) const
{
	const FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId);
	return Entry && !Entry->bHidden;
}

bool SDrawPrimitiveDebugger::IsEntryVisible(FPrimitiveRowDataPtr Data) const
{
	return Data.IsValid() ? IsEntryVisible(Data->ComponentId) : false;
}

void SDrawPrimitiveDebugger::OnRowSelectionChanged(FPrimitiveRowDataPtr InNewSelection, ESelectInfo::Type InSelectInfo)
{
	if (Selection && Selection->Data == InNewSelection)
	{
		return;
	}
	DetailView->ReleaseSelection();

	if (Selection)
	{
		Selection->bSelected = false;
	}
	
	if (InNewSelection.IsValid())
	{
		Selection = Entries.Find(InNewSelection->ComponentId);
		Selection->bSelected = true;
	}
	else
	{
		Selection = nullptr;
	}
	DetailView->UpdateSelection();
}

EVisibility SDrawPrimitiveDebugger::DetailsPanelVisibility() const
{
	return Selection && Selection->Data.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SDrawPrimitiveDebugger::OnChangeEntryPinned(ECheckBoxState State, FPrimitiveRowDataPtr Data)
{
	if (State != ECheckBoxState::Undetermined && Data.IsValid())
	{
		FPrimitiveDebuggerEntry* Entry = Entries.Find(Data->ComponentId);
		Entry->bPinned = State == ECheckBoxState::Checked;
	}
	UpdateVisibleRows();
	if (Table.IsValid())
	{
		Table->RequestListRefresh();
	}
}

bool SDrawPrimitiveDebugger::IsEntryPinned(FPrimitiveComponentId EntryId) const
{
	const FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId);
	return Entry && Entry->bPinned;
}

bool SDrawPrimitiveDebugger::IsEntryPinned(FPrimitiveRowDataPtr Data) const
{
	return Data.IsValid() ? IsEntryPinned(Data->ComponentId) : false;
}

void SDrawPrimitiveDebugger::SetForcedLODForEntry(FPrimitiveComponentId EntryId, int32 NewForcedLOD)
{
	if (FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId))
	{
		if (!Entry->Data.IsValid() || !Entry->Data->IsPrimitiveValid())
		{
			return;
		}
		if (UStaticMeshComponent* StaticMesh = Cast<UStaticMeshComponent>(Entry->Data->ComponentUObject.Get()))
		{
			if (StaticMesh->ForcedLodModel == NewForcedLOD)
			{
				return; // No change necessary
			}

			if (!Entry->bHasForcedLOD)
			{
				// Record the original desired forced LOD of the model
				Entry->DesiredForcedLOD = StaticMesh->ForcedLodModel;
				Entry->bHasForcedLOD = true;
			}
		
			StaticMesh->SetForcedLodModel(NewForcedLOD);

		}
		else if (USkinnedMeshComponent* SkinnedMesh = Cast<USkinnedMeshComponent>(Entry->Data->ComponentUObject.Get()))
		{
			if (SkinnedMesh->GetForcedLOD() == NewForcedLOD)
			{
				return; // No change necessary
			}
			if (!Entry->bHasForcedLOD)
			{
				// Record the original desired forced LOD of the model
				Entry->DesiredForcedLOD = SkinnedMesh->GetForcedLOD();
				Entry->bHasForcedLOD = true;
			}
		
			SkinnedMesh->SetForcedLOD(NewForcedLOD);

		}
		else return;
		
		if (NewForcedLOD == Entry->DesiredForcedLOD)
		{
			// The value has been reset to the desired original value, we should no longer consider consider the LOD
			// to have been modified by the debugger
			Entry->bHasForcedLOD = false;
		}
	}
}

void SDrawPrimitiveDebugger::ResetForcedLODForEntry(FPrimitiveComponentId EntryId)
{
	if (FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId))
	{
		if (!Entry->bHasForcedLOD || !Entry->Data.IsValid() || !Entry->Data->IsPrimitiveValid())
		{
			return;
		}
		if (UStaticMeshComponent* StaticMesh = Cast<UStaticMeshComponent>(Entry->Data->ComponentUObject.Get()))
		{
			StaticMesh->SetForcedLodModel(Entry->DesiredForcedLOD);
		}
		else if (USkinnedMeshComponent* SkinnedMesh = Cast<USkinnedMeshComponent>(Entry->Data->ComponentUObject.Get()))
		{
			SkinnedMesh->SetForcedLOD(Entry->DesiredForcedLOD);
		}
		Entry->bHasForcedLOD = false;
	}
}

bool SDrawPrimitiveDebugger::DoesEntryHaveForcedLOD(FPrimitiveComponentId EntryId) const
{
	if (const FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId))
	{
		bool bHasForcedLOD = false;
		if (const UStaticMeshComponent* StaticMesh = Cast<UStaticMeshComponent>(Entry->Data->ComponentUObject.Get()))
		{
			bHasForcedLOD = StaticMesh->ForcedLodModel != 0;
		}
		else if (const USkinnedMeshComponent* SkinnedMesh = Cast<USkinnedMeshComponent>(Entry->Data->ComponentUObject.Get()))
		{
			bHasForcedLOD = SkinnedMesh->GetForcedLOD() != 0;
		}
		return bHasForcedLOD;
	}
	return false;
}

void SDrawPrimitiveDebugger::SetForceDisabledNaniteForEntry(FPrimitiveComponentId EntryId, bool bForceDisableNanite)
{
	if (FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId))
	{
		if (!Entry->Data.IsValid() || !Entry->Data->IsPrimitiveValid())
		{
			return;
		}
		UStaticMeshComponent* StaticMesh = Cast<UStaticMeshComponent>(Entry->Data->ComponentUObject.Get());
		if (!IsValid(StaticMesh))
		{
			return;
		}

		if (StaticMesh->bForceDisableNanite == bForceDisableNanite)
		{
			return; // No change necessary
		}
		if (!Entry->bHasForceDisabledNanite)
		{
			// Record the original value of force disable nanite
			Entry->bDesiredForceDisabledNaniteState = StaticMesh->bForceDisableNanite;
			Entry->bHasForceDisabledNanite = true;
		}
		
		StaticMesh->SetForceDisableNanite(bForceDisableNanite);

		if (bForceDisableNanite == Entry->bDesiredForceDisabledNaniteState)
		{
			// The value has been reset to the desired original value, we should consider this value no longer modified
			Entry->bHasForceDisabledNanite = false;
		}
	}
}

void SDrawPrimitiveDebugger::SetShowDebugBoundsForEntry(FPrimitiveComponentId EntryId, bool bShowDebugBounds)
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	if (ActiveWorld.IsValid() && EntryId.IsValid())
	{
		FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId);
		if (Entry && bShowDebugBounds != Entry->bShowingDebugBounds)
		{
			if (ULineBatchComponent* const LineBatcher = ActiveWorld->GetLineBatcher(UWorld::ELineBatcherType::WorldPersistent))
			{
				if (bShowDebugBounds && Entry->Data.IsValid() && Entry->Data->IsPrimitiveValid())
				{
					const FBoxSphereBounds Bounds = Entry->Data->ComponentInterface->GetBounds();
					const FColor Color = Entry->bSelected ? FColor::Yellow : FColor::Orange;
					const float Thickness = Entry->bSelected ? 1.25f : 1.0f;
					LineBatcher->DrawBox(Bounds.Origin, Bounds.BoxExtent, Entry->Data->ComponentInterface->GetTransform().GetRotation(),
						Color, -1.0f, SDPG_World, Thickness, EntryId.PrimIDValue);
					Entry->bShowingDebugBounds = true;
					EntriesShowingDebugBounds.Add(EntryId);
				}
				else
				{
					LineBatcher->ClearBatch(EntryId.PrimIDValue);
					Entry->bShowingDebugBounds = false;
					EntriesShowingDebugBounds.Remove(EntryId);
				}
			}
		}
	}
#endif
}

bool SDrawPrimitiveDebugger::IsEntryShowingDebugBounds(FPrimitiveComponentId EntryId) const
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	const FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId);
	return Entry && Entry->bShowingDebugBounds;
#else
	return false;
#endif
}

void SDrawPrimitiveDebugger::RedrawAllDebugBounds() const
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	if (ActiveWorld.IsValid())
	{
		if (ULineBatchComponent* const LineBatcher = ActiveWorld->GetLineBatcher(UWorld::ELineBatcherType::WorldPersistent))
		{
			for (const FPrimitiveComponentId EntryId : EntriesShowingDebugBounds)
			{
				const FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId);
				if (Entry && Entry->Data.IsValid() && Entry->Data->IsPrimitiveValid())
				{
					const FBoxSphereBounds Bounds = Entry->Data->ComponentInterface->GetBounds();
					LineBatcher->ClearBatch(EntryId.PrimIDValue);
					const FColor Color = Entry->bSelected ? FColor::Yellow : FColor::Orange;
					const float Thickness = Entry->bSelected ? 1.25f : 1.0f;
					LineBatcher->DrawBox(Bounds.Origin, Bounds.BoxExtent, Entry->Data->ComponentInterface->GetTransform().GetRotation(),
						Color, -1.0f, SDPG_World, Thickness, EntryId.PrimIDValue);
				}
			}
		}
	}
#endif
}

void SDrawPrimitiveDebugger::FlushAllDebugBounds()
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	if (ActiveWorld.IsValid())
	{
		if (ULineBatchComponent* const LineBatcher = ActiveWorld->GetLineBatcher(UWorld::ELineBatcherType::WorldPersistent))
		{
			for (const FPrimitiveComponentId Entry : EntriesShowingDebugBounds)
			{
				LineBatcher->ClearBatch(Entry.PrimIDValue);
			}
		}
	}
	EntriesShowingDebugBounds.Empty();
#endif
}

void SDrawPrimitiveDebugger::SetShowDebugBonesForEntry(FPrimitiveComponentId EntryId, bool bShowDebugBones)
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	if (FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId))
	{
		if (!Entry->Data.IsValid() || !Entry->Data->IsPrimitiveValid())
		{
			return;
		}
		
		USkinnedMeshComponent* SkinnedMesh = Cast<USkinnedMeshComponent>(Entry->Data->ComponentUObject.Get());
		if (!IsValid(SkinnedMesh))
		{
			return;
		}
		
		const bool bCurrentState = SkinnedMesh->ShouldDrawDebugSkeleton();
		if (bCurrentState && !bShowDebugBones)
		{
			SkinnedMesh->SetDebugDrawColor(FLinearColor::Transparent);
			SkinnedMesh->SetDrawDebugSkeleton(false);
			SkinnedMesh->MarkRenderStateDirty();
		}
		else if (!bCurrentState && bShowDebugBones)
		{
			SkinnedMesh->SetDebugDrawColor(Entry->bSelected ? FLinearColor::Yellow : FLinearColor(FColor::Orange));
			SkinnedMesh->SetDrawDebugSkeleton(true);
			SkinnedMesh->MarkRenderStateDirty();
		}
		Entry->bShowingDebugBones = bShowDebugBones;
	}
#endif
}

bool SDrawPrimitiveDebugger::IsEntryShowingDebugBones(FPrimitiveComponentId EntryId) const
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	const FPrimitiveDebuggerEntry* Entry = Entries.Find(EntryId);
	return Entry ? Entry->bShowingDebugBones : false;
#else
	return false;
#endif
}

void SDrawPrimitiveDebugger::FlushAllDebugBones()
{
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
	for (auto& [PrimitiveId, Entry] : Entries)
	{
		if (!Entry.bShowingDebugBones || !Entry.Data.IsValid() || !Entry.Data->IsPrimitiveValid())
		{
			continue;
		}
		if (USkinnedMeshComponent* SkinnedMesh = Cast<USkinnedMeshComponent>(Entry.Data->ComponentUObject.Get()))
		{
			SkinnedMesh->SetDebugDrawColor(FLinearColor::Transparent);
			SkinnedMesh->SetDrawDebugSkeleton(false);
			SkinnedMesh->MarkRenderStateDirty();
		}
		Entry.bShowingDebugBones = false;
	}
#endif
}

void SDrawPrimitiveDebugger::FlushDebugVisualizationsForEntry(FPrimitiveComponentId EntryId)
{
	SetShowDebugBoundsForEntry(EntryId, false);
	SetShowDebugBonesForEntry(EntryId, false);
}

void SDrawPrimitiveDebugger::FlushAllDebugVisualizations()
{
	FlushAllDebugBounds();
	FlushAllDebugBones();
}

void SDrawPrimitiveDebugger::ResetDebuggerChanges()
{
	for (auto& [PrimitiveId, Entry] : Entries)
	{
		if (!Entry.Data.IsValid() || !Entry.Data->IsPrimitiveValid())
		{
			continue;
		}
		UPrimitiveComponent* Component = Entry.Data->ComponentInterface->GetUObject<UPrimitiveComponent>();
		if (Entry.bHidden)
		{
			Component->SetVisibility(true);
			Entry.bHidden = false;
			Entry.bRetainDuringRefresh = false;
		}
		if (Entry.bHasForcedLOD)
		{
			ResetForcedLODForEntry(PrimitiveId);
		}
		if (Entry.bHasForceDisabledNanite)
		{
			SetForceDisabledNaniteForEntry(PrimitiveId, Entry.bDesiredForceDisabledNaniteState);
		}
#if PRIMITIVE_DEBUGGER_SUPPORT_DEBUG_VISUALIZATIONS
		if (Entry.bShowingDebugBones)
		{
			if (USkinnedMeshComponent* SkinnedMesh = Cast<USkinnedMeshComponent>(Component))
			{
				SkinnedMesh->SetDebugDrawColor(FLinearColor::Transparent);
				SkinnedMesh->SetDrawDebugSkeleton(false);
				SkinnedMesh->MarkRenderStateDirty();
			}
			Entry.bShowingDebugBones = false;
		}
		if (Entry.bShowingDebugBounds)
		{
			if (ActiveWorld.IsValid())
			{
				if (ULineBatchComponent* const LineBatcher = ActiveWorld->GetLineBatcher(UWorld::ELineBatcherType::WorldPersistent))
				{
					LineBatcher->ClearBatch(PrimitiveId.PrimIDValue);
				}
			}
			Entry.bShowingDebugBounds = false;
		}
#endif
	}
	EntriesShowingDebugBounds.Empty();
}

bool SDrawPrimitiveDebugger::CanCaptureSingleFrame() const
{
	return IDrawPrimitiveDebugger::IsAvailable() && !IDrawPrimitiveDebugger::Get().IsLiveCaptureEnabled();
}

FReply SDrawPrimitiveDebugger::OnRefreshClick()
{
	IDrawPrimitiveDebugger::Get().CaptureSingleFrame();
	return FReply::Handled();
}

FReply SDrawPrimitiveDebugger::OnSaveClick()
{
	FViewDebugInfo::Get().DumpToCSV();
	return FReply::Handled();
}

ECheckBoxState SDrawPrimitiveDebugger::IsLiveCaptureChecked() const
{
	return IDrawPrimitiveDebugger::Get().IsLiveCaptureEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SDrawPrimitiveDebugger::OnToggleLiveCapture(ECheckBoxState state)
{
	if (state == ECheckBoxState::Checked)
	{
		IDrawPrimitiveDebugger::Get().EnableLiveCapture();
	}
	else if (state == ECheckBoxState::Unchecked)
	{
		IDrawPrimitiveDebugger::Get().DisableLiveCapture();
	}
}

FPrimitiveRowDataPtr SDrawPrimitiveDebugger::GetCurrentSelection() const
{
	return Selection ? Selection->Data : nullptr;
}

FPrimitiveComponentId SDrawPrimitiveDebugger::GetCurrentSelectionId() const
{
	return Selection ? Selection->Data->ComponentId : FPrimitiveComponentId();
}

void SDrawPrimitiveDebugger::HandleActorCleanup(AActor* Actor)
{
	TArray<UPrimitiveComponent*> PrimitiveComponents;
	TSet<FPrimitiveComponentId> PrimitiveComponentIds;
	
	Actor->GetComponents<UPrimitiveComponent>(PrimitiveComponents);
	
	PrimitiveComponentIds.Reserve(PrimitiveComponents.Num());
	
	for (const UPrimitiveComponent* Component : PrimitiveComponents)
	{
		FPrimitiveComponentId ComponentId = Component->GetPrimitiveSceneId();
		FlushDebugVisualizationsForEntry(ComponentId);
		Entries.Remove(ComponentId);
		PrimitiveComponentIds.Add(ComponentId);
	}
	
	auto CheckForMatch = [PrimitiveComponentIds](FPrimitiveRowDataPtr Entry) -> bool
	{
		return PrimitiveComponentIds.Contains(Entry->ComponentId);
	};
	
	AvailableEntries.RemoveAll(CheckForMatch);
	VisibleEntries.RemoveAll(CheckForMatch);
	
	if (Selection && Selection->Data.IsValid() && PrimitiveComponentIds.Contains(Selection->Data->ComponentId))
	{
		this->OnRowSelectionChanged(nullptr, ESelectInfo::Direct);
	}
	
	if (Table.IsValid())
	{
		Table->RequestListRefresh();
	}
}

SDrawPrimitiveDebugger::FPrimitiveDebuggerEntry::FPrimitiveDebuggerEntry(const FPrimitiveRowDataPtr& Data) : Data(Data)
{
	bHidden = false;
	bPinned = false;
	bSelected = false;
	bShowingDebugBones = false;
	bShowingDebugBounds = false;
	bHasForceDisabledNanite = false;
	bHasForcedLOD = false;
	bRetainDuringRefresh = false;
	DesiredForcedLOD = 0;
	bDesiredForceDisabledNaniteState = false;
}

SDrawPrimitiveDebugger::FPrimitiveDebuggerEntry::FPrimitiveDebuggerEntry(const FViewDebugInfo::FPrimitiveInfo& Primitive)
	: FPrimitiveDebuggerEntry(MakeShared<const FViewDebugInfo::FPrimitiveInfo>(Primitive))
{
}

void SDrawPrimitiveDebuggerListViewRow::Construct(const FArguments& InArgs,
                                                  const TSharedRef<STableViewBase>& InOwnerTableView)
{
	RowDataPtr = InArgs._RowDataPtr;
	DrawPrimitiveDebugger = InArgs._DrawPrimitiveDebugger;
	SMultiColumnTableRow<FPrimitiveRowDataPtr>::Construct(
		FSuperRowType::FArguments(),
		InOwnerTableView
	);
}

TSharedRef<SWidget> SDrawPrimitiveDebuggerListViewRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	const TSharedPtr<SDrawPrimitiveDebugger> DrawPrimitiveDebuggerPtr = DrawPrimitiveDebugger.Pin();
	return (DrawPrimitiveDebuggerPtr.IsValid())
		? MakeCellWidget(IndexInList, ColumnName)
		: SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDrawPrimitiveDebuggerListViewRow::MakeCellWidget(const int32 InRowIndex, const FName& InColumnId)
{
	static const FName VisibilityColumn("Visible");
	static const FName PinColumn("Pin");
	static const FName NameColumn("Name");
	static const FName ActorClassColumn("ActorClass");
	static const FName ActorColumn("Actor");

	static const FMargin Margin(5, 2, 5, 2);
	const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FSlateFontInfo FontInfo = FSlateFontInfo(FCoreStyle::GetDefaultFont(), UDrawPrimitiveDebuggerUserSettings::GetFontSize());
	
	SCOPE_CYCLE_COUNTER(STAT_PrimitiveDebuggerMakeCell);
	SDrawPrimitiveDebugger* DrawPrimitiveDebuggerPtr = DrawPrimitiveDebugger.Pin().Get();
	if (DrawPrimitiveDebuggerPtr && RowDataPtr.IsValid())
	{
		FText Value;
		if (InColumnId.IsEqual(VisibilityColumn))
		{
			SCOPE_CYCLE_COUNTER(STAT_PrimitiveDebuggerMakeCellVisible);
			return SNew(SBox)
				.Padding(Margin)
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
						.IsChecked(this, &SDrawPrimitiveDebuggerListViewRow::IsVisible)
						.OnCheckStateChanged(DrawPrimitiveDebuggerPtr, &SDrawPrimitiveDebugger::OnChangeEntryVisibility, RowDataPtr)
						.HAlign(HAlign_Center)
				];
		}
		if (InColumnId.IsEqual(PinColumn))
		{
			SCOPE_CYCLE_COUNTER(STAT_PrimitiveDebuggerMakeCellPinned);
			return SNew(SBox)
				.Padding(Margin)
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
						.IsChecked(this, &SDrawPrimitiveDebuggerListViewRow::IsPinned)
						.OnCheckStateChanged(DrawPrimitiveDebuggerPtr, &SDrawPrimitiveDebugger::OnChangeEntryPinned, RowDataPtr)
						.HAlign(HAlign_Center)
				];
		}
		if (InColumnId.IsEqual(NameColumn))
		{
			SCOPE_CYCLE_COUNTER(STAT_PrimitiveDebuggerMakeCellName);
			Value = FText::FromString(RowDataPtr->Name);
		}
		else if (InColumnId.IsEqual(ActorClassColumn))
		{
			SCOPE_CYCLE_COUNTER(STAT_PrimitiveDebuggerMakeCellActorClass);
			Value = RowDataPtr->Owner.IsValid() && IsValid(RowDataPtr->Owner->GetClass()) ?
				FText::FromString(RowDataPtr->Owner->GetClass()->GetName()) :
				InvalidTextValue;
		}
		else if (InColumnId.IsEqual(ActorColumn))
		{
			SCOPE_CYCLE_COUNTER(STAT_PrimitiveDebuggerMakeCellActor);
			Value = RowDataPtr->Owner.IsValid() ?
				FText::FromString(RowDataPtr->GetOwnerName()) :
				InvalidTextValue;
		}
		else
		{
			// Invalid Column name
			return SNullWidget::NullWidget;
		}
		return SNew(SBox)
			.Padding(Margin)
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Text(Value)
				.ToolTipText(Value)
				.Font(FontInfo)
				.IsEnabled(DrawPrimitiveDebuggerPtr, &SDrawPrimitiveDebugger::IsEntryVisible, RowDataPtr)
				.Justification(ETextJustify::Left)
				.HighlightText(DrawPrimitiveDebuggerPtr, &SDrawPrimitiveDebugger::GetFilterText)
			];
	}
	return SNullWidget::NullWidget;
}

ECheckBoxState SDrawPrimitiveDebuggerListViewRow::IsVisible() const
{
	const SDrawPrimitiveDebugger* DrawPrimitiveDebuggerPtr = DrawPrimitiveDebugger.Pin().Get();
	return DrawPrimitiveDebuggerPtr && DrawPrimitiveDebuggerPtr->IsEntryVisible(RowDataPtr) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState SDrawPrimitiveDebuggerListViewRow::IsPinned() const
{
	const SDrawPrimitiveDebugger* DrawPrimitiveDebuggerPtr = DrawPrimitiveDebugger.Pin().Get();
	return DrawPrimitiveDebuggerPtr && DrawPrimitiveDebuggerPtr->IsEntryPinned(RowDataPtr) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#endif // WITH_PRIMITIVE_DEBUGGER

#undef LOCTEXT_NAMESPACE