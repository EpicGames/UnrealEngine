// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SSearchBox.h"
#include "PrimitiveComponentId.h"
#include "ViewDebug.h"
#include "DrawPrimitiveDebugger.h"

#if WITH_PRIMITIVE_DEBUGGER

class UStaticMeshComponent;
class USkinnedMeshComponent;
class UMaterialInterface;

class SDrawPrimitiveDebugger;

typedef TSharedPtr<const FViewDebugInfo::FPrimitiveInfo> FPrimitiveRowDataPtr;

class SPrimitiveDebuggerDetailView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPrimitiveDebuggerDetailView) { }
	/** The primitive we're currently focused on. */
	SLATE_ARGUMENT(TWeakPtr<SDrawPrimitiveDebugger>, PrimitiveDebugger)
	SLATE_END_ARGS()

	virtual ~SPrimitiveDebuggerDetailView() override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void UpdateSelection();
	void ReleaseSelection();
	
private:
	
	TWeakPtr<SDrawPrimitiveDebugger> PrimitiveDebugger;
	TSharedPtr<SVerticalBox> DetailPropertiesWidget;
	TSharedPtr<SVerticalBox> MaterialsWidget;
	TSharedPtr<SVerticalBox> AdvancedOptionsWidget;
	TSharedPtr<SVerticalBox> SkeletalMeshDetailsWidget;

	bool bSelectionSupportsNanite = false;
	bool bSelectionIsNaniteEnabledThisFrame = false;

	FText SelectedActorName;
	FText SelectedActorPath;
	FText SelectedActorClassName;
	FText SelectedActorClassPath;
	FText SelectedPrimitiveType;

	TWeakObjectPtr<UStaticMeshComponent> SelectedAsStaticMesh = nullptr;
	TWeakObjectPtr<USkinnedMeshComponent> SelectedAsSkinnedMesh = nullptr;
	UClass* SelectedComponentType = nullptr;

	const FPrimitiveLODStats* CurrentLOD = nullptr;
	int32 PlayerIndex = 0;
	int32 ViewIndex = 0;

	FText GetSelectedPrimitiveName() const;
	FText GetSelectedPrimitiveType() const;
	FText GetSelectedActorName() const;
	FText GetSelectedActorToolTip() const;
	FText GetSelectedActorClassName() const;
	FText GetSelectedActorClassToolTip() const;
	FText GetSelectedPrimitiveNaniteEnabled() const;
	FText GetSelectedPrimitiveSupportsNanite() const;
	FText GetSelectedDrawCallCount() const;
	FText GetSelectedLocation() const;
	FText GetSelectedLOD() const;
	FText GetSelectedNumLODs() const;
	TOptional<int> GetSelectedLODValue() const;
	TOptional<int> GetSelectedForcedLODValue() const;
	TOptional<int> GetSelectedNumLODsValue() const;
	TOptional<int> GetSelectedForcedLODSliderMaxValue() const;
	FText GetSelectedTriangleCount() const;
	FText GetSelectedBoneCount() const;

	void GenerateDetailPanelEntry(const FText& Label,
		FText (SPrimitiveDebuggerDetailView::* ValueGetter)() const,
		EVisibility (SPrimitiveDebuggerDetailView::* VisibilityGetter)() const = nullptr,
		FText (SPrimitiveDebuggerDetailView::* TooltipGetter)() const = nullptr,
		bool bSupportHighlighting = false) const;
	
	TSharedRef<SVerticalBox> GetSelectedMaterialsWidget();
	TSharedRef<SVerticalBox> GetAdvancedOptionsWidget();
	
	EVisibility OptionVisibilityForceLOD() const;
	EVisibility OptionVisibilityForceDisableNanite() const;

	ECheckBoxState ForceLODState() const;
	bool IsForceLODIndexSliderEnabled() const;
	void OnToggleForceLOD(ECheckBoxState state);
	void HandleForceLOD(int ForcedLOD);
	
	ECheckBoxState ShowDebugBoundsState() const;
	void OnToggleDebugBounds(ECheckBoxState state);
	
	ECheckBoxState ForceDisableNaniteState() const;
	void OnToggleForceDisableNanite(ECheckBoxState state);

	EVisibility StaticMeshDataVisibility() const;
	EVisibility SkeletalMeshDataVisibility() const;
	ECheckBoxState ShowDebugBonesState() const;
	void OnToggleDebugBones(ECheckBoxState state);
	
	EVisibility NaniteDataVisibility() const;
	EVisibility NonNaniteDataVisibility() const;
	
	void CreateMaterialEntry(const UMaterialInterface* MI, int Index, bool bIsOverlay = false);
};

class SDrawPrimitiveDebugger : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDrawPrimitiveDebugger)
		{
		}

	SLATE_END_ARGS()

	virtual ~SDrawPrimitiveDebugger() override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
	FText GetFilterText() const;
	void OnFilterTextChanged(const FText& InFilterText);
	void OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);
	
	TSharedRef<ITableRow> MakeRowWidget(FPrimitiveRowDataPtr InRowDataPtr, const TSharedRef<STableViewBase>& OwnerTable);
	void OnRowSelectionChanged(FPrimitiveRowDataPtr InNewSelection, ESelectInfo::Type InSelectInfo);
	void UpdateVisibleRows();
	void SortRows();
	void Refresh();
	void ClearAllEntries();
	void SetActiveWorld(UWorld* World);
	void RemoveEntry(FPrimitiveRowDataPtr Entry);
	void AddColumn(const FText& Name, const FName& ColumnId);
	
	void OnChangeEntryVisibility(ECheckBoxState state, FPrimitiveRowDataPtr Data);
	bool IsEntryVisible(FPrimitiveComponentId EntryId) const;
	bool IsEntryVisible(FPrimitiveRowDataPtr Data) const;
	void OnChangeEntryPinned(ECheckBoxState State, FPrimitiveRowDataPtr Data);
	bool IsEntryPinned(FPrimitiveComponentId EntryId) const;
	bool IsEntryPinned(FPrimitiveRowDataPtr Data) const;

	void SetForcedLODForEntry(FPrimitiveComponentId EntryId, int32 NewForcedLOD);
	void ResetForcedLODForEntry(FPrimitiveComponentId EntryId);
	bool DoesEntryHaveForcedLOD(FPrimitiveComponentId EntryId) const;
	void SetForceDisabledNaniteForEntry(FPrimitiveComponentId EntryId, bool bForceDisableNanite);
	
	void SetShowDebugBoundsForEntry(FPrimitiveComponentId EntryId, bool bShowDebugBounds);
	bool IsEntryShowingDebugBounds(FPrimitiveComponentId EntryId) const;
	void RedrawAllDebugBounds() const;
	void FlushAllDebugBounds();
	
	void SetShowDebugBonesForEntry(FPrimitiveComponentId EntryId, bool bShowDebugBones);
	bool IsEntryShowingDebugBones(FPrimitiveComponentId EntryId) const;
	void FlushAllDebugBones();

	void FlushDebugVisualizationsForEntry(FPrimitiveComponentId EntryId);
	void FlushAllDebugVisualizations();

	/** Resets any changes made to the scene by the debugger and clears all debugger related debug visualizations. */
	void ResetDebuggerChanges();
	
	bool CanCaptureSingleFrame() const;
	FReply OnRefreshClick();
	FReply OnSaveClick();
	ECheckBoxState IsLiveCaptureChecked() const;
	void OnToggleLiveCapture(ECheckBoxState State);
	FPrimitiveRowDataPtr GetCurrentSelection() const;
	FPrimitiveComponentId GetCurrentSelectionId() const;
	EVisibility DetailsPanelVisibility() const;

private:

	struct FPrimitiveDebuggerEntry
	{
		FPrimitiveRowDataPtr Data;
		
		uint8 bHidden : 1;
		uint8 bPinned : 1;
		uint8 bSelected : 1;
		uint8 bShowingDebugBounds : 1;
		uint8 bShowingDebugBones : 1;
		uint8 bHasForcedLOD : 1;
		uint8 bHasForceDisabledNanite : 1;
		uint8 bRetainDuringRefresh : 1;

		// The below values are only updated if the debugger modifies this primitive
		
		int32 DesiredForcedLOD;
		bool bDesiredForceDisabledNaniteState;

		FPrimitiveDebuggerEntry(const FPrimitiveRowDataPtr& Data = nullptr);
		FPrimitiveDebuggerEntry(const FViewDebugInfo::FPrimitiveInfo& Primitive);
		
		FPrimitiveDebuggerEntry(const FPrimitiveDebuggerEntry& Other) = default;
		FPrimitiveDebuggerEntry(FPrimitiveDebuggerEntry&& Other) noexcept = default;

		FPrimitiveDebuggerEntry& operator=(const FPrimitiveDebuggerEntry& RHS) = default;
		FPrimitiveDebuggerEntry& operator=(FPrimitiveDebuggerEntry&& RHS) = default;
	};

	TMap<FPrimitiveComponentId, FPrimitiveDebuggerEntry> Entries;
	

	TSharedPtr<SPrimitiveDebuggerDetailView> DetailView;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SListView<FPrimitiveRowDataPtr>> Table;
	FText FilterText;
	TSharedPtr<SHeaderRow> ColumnHeader;
	TArray<FPrimitiveRowDataPtr> AvailableEntries;
	TArray<FPrimitiveRowDataPtr> VisibleEntries;
	TWeakObjectPtr<UWorld> ActiveWorld;
	FDelegateHandle ActorComponentsUnregisteredHandle;
	
	FPrimitiveDebuggerEntry* Selection = nullptr;
	
	// We keep an extra list of primitives with debug bounds to avoid unnecessary cycles during Tick when updating bounds lines
	TSet<FPrimitiveComponentId> EntriesShowingDebugBounds;

	void HandleActorCleanup(AActor* Actor);
};

/**
 * A widget to represent a row in a Data Table Editor widget. This widget allows us to do things like right-click
 * and take actions on a particular row of a Data Table.
 */
class SDrawPrimitiveDebuggerListViewRow : public SMultiColumnTableRow<FPrimitiveRowDataPtr>
{
public:

	SLATE_BEGIN_ARGS(SDrawPrimitiveDebuggerListViewRow)
	{
	}
	/** The owning object. This allows us access to the actual data table being edited as well as some other API functions. */
	SLATE_ARGUMENT(TWeakPtr<SDrawPrimitiveDebugger>, DrawPrimitiveDebugger)
		/** The primitive we're working with to allow us to get naming information. */
		SLATE_ARGUMENT(FPrimitiveRowDataPtr, RowDataPtr)
		SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	TSharedRef<SWidget> MakeCellWidget(const int32 InRowIndex, const FName& InColumnId);
	
	ECheckBoxState IsVisible() const;
	ECheckBoxState IsPinned() const;

	FPrimitiveRowDataPtr RowDataPtr;
	TWeakPtr<SDrawPrimitiveDebugger> DrawPrimitiveDebugger;
};

#endif