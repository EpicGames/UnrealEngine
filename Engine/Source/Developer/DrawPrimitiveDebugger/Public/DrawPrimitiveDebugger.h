// Copyright Epic Games, Inc. All Rights Reserved.

// DRAW PRIMITIVE DEBUGGER
// 
// This tool allows easy on screen debugging of graphics data, primarily on-screen primitives
// 

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define WITH_PRIMITIVE_DEBUGGER !UE_BUILD_SHIPPING && !UE_SERVER

DECLARE_LOG_CATEGORY_EXTERN(LogDrawPrimitiveDebugger, All, All);

class IDrawPrimitiveDebugger : public IModuleInterface
{

public:
	/**
	 * Singleton-like access to this module's interface. This is just for convenience!
	 * Beware of calling this during the shutdown phase, though. Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IDrawPrimitiveDebugger& Get()
	{
		return FModuleManager::LoadModuleChecked< IDrawPrimitiveDebugger >("DrawPrimitiveDebugger");
	}

	/**
	 * Checks to see if this module is loaded and ready. It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("DrawPrimitiveDebugger");
	}

	/**
	 * Instructs the renderer to capture a snapshot for the debugger on the next frame.
	 */
	virtual void CaptureSingleFrame() = 0;

	/**
	 * Is live data capture enabled for the debugger? This implies that the renderer will capture debug data each frame.
	 * @return True if live capture is enabled.
	 */
	virtual bool IsLiveCaptureEnabled() const = 0;
	
	/**
	 * Enables capturing debug data each frame.
	 */
	virtual void EnableLiveCapture() = 0;
	/**
	 * Disables capturing debug data each frame.
	 */
	virtual void DisableLiveCapture() = 0;

	/**
	 * Discards the current snapshot data in memory and resets the debugger window.
	 */
	virtual void DiscardCaptureData() = 0;

	/**
	 * Opens the graphics debugger window if it is available.
	 */
	virtual void OpenDebugWindow() = 0;
	/**
	 * Closes the graphics debugger window if it is currently open.
	 */
	virtual void CloseDebugWindow() = 0;
	
};
