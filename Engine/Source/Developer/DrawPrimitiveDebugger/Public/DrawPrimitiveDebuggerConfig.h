// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "DrawPrimitiveDebuggerConfig.generated.h"

#define UE_API DRAWPRIMITIVEDEBUGGER_API

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings, meta = (DisplayName = "Draw Primitive Debugger"))
class UDrawPrimitiveDebuggerUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()
protected:
	virtual FName GetCategoryName() const override { return TEXT("Advanced"); }

public:
	
	static int32 GetFontSize() { return GetDefault<UDrawPrimitiveDebuggerUserSettings>()->FontSize; }
	static UE_API void SetFontSize(const int32 InFontSize);

protected:
	
	/** Font Size used by Draw Primitive Debugger */ 
	UPROPERTY(config, EditAnywhere, Category = DrawPrimitiveDebugger)
	int32 FontSize = 10;
};

#undef UE_API
